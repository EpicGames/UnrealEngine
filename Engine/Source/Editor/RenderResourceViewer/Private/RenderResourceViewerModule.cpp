// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderResourceViewerModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "SRenderResourceViewer.h"

#define LOCTEXT_NAMESPACE "RenderResourceViewerModule"

IMPLEMENT_MODULE(FRenderResourceViewerModule, RenderResourceViewer);

FName RenderResourceViewerTabId("RenderResourceViewer");

void FRenderResourceViewerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(RenderResourceViewerTabId, FOnSpawnTab::CreateRaw(this, &FRenderResourceViewerModule::CreateTab))
		.SetDisplayName(NSLOCTEXT("RenderResourceViewer", "TabTitle", "Render Resource Viewer"))
		.SetTooltipText(NSLOCTEXT("RenderResourceViewer", "TooltipText", "Open the render resource viewer."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"));
}

void FRenderResourceViewerModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(RenderResourceViewerTabId);
	}
}

TSharedRef<SDockTab> FRenderResourceViewerModule::CreateTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab).TabRole(ETabRole::NomadTab);
	TSharedRef<SRenderResourceViewerWidget> WindowRef = SNew(SRenderResourceViewerWidget, DockTab, Args.GetOwnerWindow());
	DockTab->SetContent(WindowRef);
	AssignWindow(WindowRef);
	return DockTab;
}

#undef LOCTEXT_NAMESPACE
