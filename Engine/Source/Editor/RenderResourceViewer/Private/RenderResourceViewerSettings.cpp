// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderResourceViewerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderResourceViewerSettings)

URenderResourceViewerSettings::URenderResourceViewerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ 
}
