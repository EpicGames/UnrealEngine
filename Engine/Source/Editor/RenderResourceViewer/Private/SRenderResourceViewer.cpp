// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenderResourceViewer.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "RHI.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Developer/TreeMap/Public/STreeMap.h"
#include "RenderResourceViewerSettings.h"
#include "Toolkits/GlobalEditorCommonCommands.h"

#define LOCTEXT_NAMESPACE "SRenderResourceView"

namespace RenderResourceViewerInternal
{
	const FName ColumnName("Name");
	const FName ColumnType("Type");
	const FName ColumnSize("Size");
	const FName ColumnFlags("Flags");
	const FName ColumnOwner("Owner");
	
	// Format the size to nearest size unit of Byte/KB/MB
	FString GetFormatedSize(uint64 SizeInBytes)
	{
		const uint64 KB = 1024;
		const uint64 MB = KB * KB;
		if (SizeInBytes < KB)
		{
			return FString::FromInt(SizeInBytes) + TEXT(" B");
		}
		else if (SizeInBytes < MB)
		{
			uint32 SizeInKB = SizeInBytes / KB;
			return FString::FromInt(SizeInKB) + TEXT(" KB");
		}
		else
		{
			uint32 SizeInMB = SizeInBytes / MB;
			return FString::FromInt(SizeInMB) + TEXT(" MB");
		}
	}
	
	struct FResourceInfoRow : public SMultiColumnTableRow<TSharedPtr<FRHIResourceStats>>
	{
		SLATE_BEGIN_ARGS(FResourceInfoRow) {}
		SLATE_END_ARGS()
	
		TSharedPtr<FRHIResourceStats> Info;
	
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FRHIResourceStats> InItem)
		{
			Info = InItem;
			SMultiColumnTableRow<TSharedPtr<FRHIResourceStats>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}
	
		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& Column)
		{
			if (Column == RenderResourceViewerInternal::ColumnName)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Info->Name));
			}
			else if (Column == RenderResourceViewerInternal::ColumnType)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(Info->Type));
			}
			else if (Column == RenderResourceViewerInternal::ColumnSize)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(RenderResourceViewerInternal::GetFormatedSize(Info->SizeInBytes)));
			}
			else if (Column == RenderResourceViewerInternal::ColumnFlags)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(Info->Flags));
			}
			else if (Column == RenderResourceViewerInternal::ColumnOwner)
			{
				return SNew(STextBlock)
					.Text(FText::FromName(Info->OwnerName));
			}
			return SNullWidget::NullWidget;
		}
	};
}

#define RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(DisplayText, Tooltip, Options, SelectedItem, OnSelectionChangedFunc) \
	+ SHorizontalBox::Slot() \
		.AutoWidth() \
		.Padding(FMargin(0.0f, 2.0f, 2.0f, 0.0f)) \
		[ \
			SNew(STextBlock) \
			.Text(DisplayText) \
			.ToolTipText(Tooltip) \
		] \
	+ SHorizontalBox::Slot() \
		.AutoWidth() \
		.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f)) \
		[ \
			SNew(STextComboBox) \
			.ToolTipText(Tooltip) \
			.OptionsSource(&Options) \
			.InitiallySelectedItem(SelectedItem) \
			.OnSelectionChanged(this, &OnSelectionChangedFunc) \
		]

#define RENDER_RESOURCE_VIEWER_ADD_COLUMN(Name, Width, Label) \
	+ SHeaderRow::Column(Name) \
		.FillWidth(Width) \
		.DefaultLabel(Label) \
		.SortMode(this, &SRenderResourceViewerWidget::GetColumnSortMode, Name) \
		.OnSort(this, &SRenderResourceViewerWidget::OnColumnSortModeChanged)

SRenderResourceViewerWidget::SRenderResourceViewerWidget()
: TreeMapWidget(nullptr),
  RootTreeMapNode(new FTreeMapNodeData())
{

}

void SRenderResourceViewerWidget::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	SortByColumn = RenderResourceViewerInternal::ColumnSize;
	SortMode = EColumnSortMode::Descending;

	ComboBoxNames.Empty();
	ComboBoxNames.Add(MakeShareable(new FString(TEXT("Any"))));
	ComboBoxNames.Add(MakeShareable(new FString(TEXT("Yes"))));
	ComboBoxNames.Add(MakeShareable(new FString(TEXT("No"))));

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(0.5f)
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(TreeMapWidget, STreeMap, RootTreeMapNode.ToSharedRef(), nullptr)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.0f, 2.0f, 10.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FilterText", "FilterByFlags:"))
					]

					// A row of comboboxes to filter resource list by flags
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("ResidentText", "Resident"), LOCTEXT("ResidentTooltip", "Resource is accessible by GPU, and not evicted (unused). 'Any' will show all resources, 'Yes' only resources which are resident and 'No' resources which are evicted."), 
						ComboBoxNames, ComboBoxTypeToName(ShowResident), SRenderResourceViewerWidget::OnResidentComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("TransientText", "Transient"), LOCTEXT("TransientTooltip", "Resource is only allocated during the duration of the render passes where it's active and will share underlying memory with other resources in the frame. 'Any' will show all resources, 'Yes' only resources which are transient and 'No' resources which are not transient."), 
						ComboBoxNames, ComboBoxTypeToName(ShowTransient), SRenderResourceViewerWidget::OnTransientComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("StreamingText", "Streaming"), LOCTEXT("StreamingTooltip", "Resource is a streamable texture. 'Any' will show all resources, 'Yes' only resources which are streaming and 'No' resources which are not streaming."), 
						ComboBoxNames, ComboBoxTypeToName(ShowStreaming), SRenderResourceViewerWidget::OnStreamingComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("RTText", "RT"), LOCTEXT("RTTooltip", "Resource can be written to as a Render Target buffer by GPU. 'Any' will show all resources, 'Yes' only resources which can be used as a RenderTarget and 'No' resources which can't be used as a RenderTarget."), 
						ComboBoxNames, ComboBoxTypeToName(ShowRT), SRenderResourceViewerWidget::OnRTComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("DSText", "DS"), LOCTEXT("DSTooltip", "Resource can be written to as a Depth Stencil buffer by GPU. 'Any' will show all resources, 'Yes' only resources which can be used as a DepthStencil Target and 'No' resources which can't be used as a DepthStencil Target."), 
						ComboBoxNames, ComboBoxTypeToName(ShowDS), SRenderResourceViewerWidget::OnDSComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("UAVText", "UAV"), LOCTEXT("UAVTooltip", "Resource supports Unordered Access View which allows temporally unordered read/write access from multiple GPU threads without generating memory conflicts. . 'Any' will show all resources, 'Yes' only resources which can be used as a UAV and 'No' resources which can't be used as a UAV."), 
						ComboBoxNames, ComboBoxTypeToName(ShowUAV), SRenderResourceViewerWidget::OnUAVComboboxChanged)
					RENDER_RESOURCE_VIEWER_ADD_COMBOBOX(LOCTEXT("RTASText", "RTAS"), LOCTEXT("RTASTooltip", "Resource is a Ray Tracing Acceleration Structure. 'Any' will show all resources, 'Yes' only resources which are RTAS and 'No' resources which are not RTAS."), 
						ComboBoxNames, ComboBoxTypeToName(ShowRTAS), SRenderResourceViewerWidget::OnRTASComboboxChanged)

					// Refresh button to update the resource list
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					[
						SNew(SButton)
						.Text(LOCTEXT("RefreshText", "Refresh"))
						.ToolTipText(LOCTEXT("RefreshTooltip", "Refresh the resource list"))
						.OnClicked(this, &SRenderResourceViewerWidget::OnRefreshButtonClicked)
					]
				]

				// Text filter box to search by resource name
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(FilterTextBox, SSearchBox)
					.HintText(LOCTEXT("FilterTextBoxHint", "Search resources by name or owner."))
					.ToolTipText(LOCTEXT("FilterTextBoxToolTip", "Type here to filter the list of render resources by name or owner."))
					.OnTextChanged(this, &SRenderResourceViewerWidget::FilterTextBox_OnTextChanged)
				]

				// Display total resource count and size in text
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TotalResourceCountText", "Total Resource Count:"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(this, &SRenderResourceViewerWidget::GetResourceCountText)
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TotalResourceSizeText", "Total Resource Size:"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &SRenderResourceViewerWidget::GetResourceSizeText)
					]
				]

				// Display resource list
				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(ResourceListView, SListView<TSharedPtr<FRHIResourceStats>>)
					.ScrollbarVisibility(EVisibility::Visible)
					.ToolTipText(LOCTEXT("ResourceListViewToolTip", "CTRL+B or use right click menu to find the asset in Content Browser"))
					.ListItemsSource(&ResourceInfos)
					.SelectionMode(ESelectionMode::SingleToggle)
					.OnGenerateRow(this, &SRenderResourceViewerWidget::HandleResourceGenerateRow)
					.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &SRenderResourceViewerWidget::OpenContextMenu))
					.HeaderRow
					(
						SNew(SHeaderRow)

						RENDER_RESOURCE_VIEWER_ADD_COLUMN(RenderResourceViewerInternal::ColumnName, 0.3f, LOCTEXT("NameColumn", "Resource Name"))
						RENDER_RESOURCE_VIEWER_ADD_COLUMN(RenderResourceViewerInternal::ColumnType, 0.1f, LOCTEXT("TypeColumn", "Type"))
						RENDER_RESOURCE_VIEWER_ADD_COLUMN(RenderResourceViewerInternal::ColumnSize, 0.1f, LOCTEXT("SizeColumn", "Size"))
						RENDER_RESOURCE_VIEWER_ADD_COLUMN(RenderResourceViewerInternal::ColumnFlags, 0.15f, LOCTEXT("FlagsColumn", "Flags"))
						RENDER_RESOURCE_VIEWER_ADD_COLUMN(RenderResourceViewerInternal::ColumnOwner, 0.4f, LOCTEXT("OwnerColumn", "Owner"))
					)
				]
			]
		]
	];

	InitCommandList();

	RefreshNodes(true);
}

SRenderResourceViewerWidget::EComboBoxType SRenderResourceViewerWidget::ComboBoxNameToType(TSharedPtr<FString> Value) const
{
	int32 Index = ComboBoxNames.Find(Value);
	if (Index != INDEX_NONE)
	{
		return (EComboBoxType)Index;		
	}
	return EComboBoxType::Any;
}

bool SRenderResourceViewerWidget::ShouldShow(EComboBoxType FilterType, bool bValue) const
{
	switch (FilterType)
	{
	case EComboBoxType::Any:
	{
		return true;
	}
	case EComboBoxType::No:
	{
		return !bValue;
	}
	case EComboBoxType::Yes:
	{
		return bValue;
	}
	}

	return true;
}

void SRenderResourceViewerWidget::RefreshNodes(bool bUpdateRHIResources)
{
	const int32 NumberOfResourcesToShow = -1;

	// Retrieve the whole list of RHI tracked resources
#if RHI_ENABLE_RESOURCE_INFO
	if (bUpdateRHIResources)
	{
		RHIResources.Reset();
		RHIGetTrackedResourceStats(RHIResources);
	}
#endif

	// Filter the resource list by search keyword and flags
	ResourceInfos.Reset();
	TotalResourceSize = 0;
	for (const TSharedPtr<FRHIResourceStats>& Info : RHIResources)
	{
		bool bContainsFilterText = FilterText.IsEmpty() || Info->Name.ToString().Contains(FilterText.ToString()) || Info->OwnerName.ToString().Contains(FilterText.ToString());
		bool bContainsFilterFlags = true;
		// If the resource has a flag set and its matching check box is un-ticked, exclude from display.
		// bMarkedForDelete resources are excluded from display
		if (!ShouldShow(ShowResident, Info->bResident) ||
			!ShouldShow(ShowTransient, Info->bTransient) ||
			!ShouldShow(ShowStreaming, Info->bStreaming) ||
			!ShouldShow(ShowRT, Info->bRenderTarget) ||
			!ShouldShow(ShowDS, Info->bDepthStencil) ||
			!ShouldShow(ShowUAV, Info->bUnorderedAccessView) ||
			!ShouldShow(ShowRTAS, Info->bRayTracingAccelerationStructure) ||
			Info->bMarkedForDelete)
		{
			bContainsFilterFlags = false;
		}
		if (bContainsFilterText && bContainsFilterFlags)
		{
			ResourceInfos.Add(Info);
			TotalResourceSize += Info->SizeInBytes;
		}
	}

	TotalResourceCount = ResourceInfos.Num();

	// Sort the resource list based on columns.
	{
		if (SortByColumn == RenderResourceViewerInternal::ColumnName)
		{
			ResourceInfos.Sort([this](const TSharedPtr<FRHIResourceStats>& A, const TSharedPtr<FRHIResourceStats>& B) {
				if (SortMode == EColumnSortMode::Ascending) return (A->Name.LexicalLess(B->Name));
				else if (SortMode == EColumnSortMode::Descending) return (!A->Name.LexicalLess(B->Name));
				return true;
			});
		}
		else if (SortByColumn == RenderResourceViewerInternal::ColumnType)
		{
			ResourceInfos.Sort([this](const TSharedPtr<FRHIResourceStats>& A, const TSharedPtr<FRHIResourceStats>& B) {
				if (SortMode == EColumnSortMode::Ascending) return (A->Type < B->Type);
				else if (SortMode == EColumnSortMode::Descending) return (A->Type >= B->Type);
				return true;
			});
		}
		else if (SortByColumn == RenderResourceViewerInternal::ColumnSize)
		{
			ResourceInfos.Sort([this](const TSharedPtr<FRHIResourceStats>& A, const TSharedPtr<FRHIResourceStats>& B) {
				if (SortMode == EColumnSortMode::Ascending) return (A->SizeInBytes < B->SizeInBytes);
				else if (SortMode == EColumnSortMode::Descending) return (A->SizeInBytes >= B->SizeInBytes);
				return true;
			});
		}
		else if (SortByColumn == RenderResourceViewerInternal::ColumnFlags)
		{
			ResourceInfos.Sort([this](const TSharedPtr<FRHIResourceStats>& A, const TSharedPtr<FRHIResourceStats>& B) {
				if (SortMode == EColumnSortMode::Ascending) return (A->Flags < B->Flags);
				else if (SortMode == EColumnSortMode::Descending) return (A->Flags >= B->Flags);
				return true;
			});
		}
		else if (SortByColumn == RenderResourceViewerInternal::ColumnOwner)
		{
			ResourceInfos.Sort([this](const TSharedPtr<FRHIResourceStats>& A, const TSharedPtr<FRHIResourceStats>& B) {
				if (SortMode == EColumnSortMode::Ascending) return (A->OwnerName.LexicalLess(B->OwnerName));
				else if (SortMode == EColumnSortMode::Descending) return (!A->OwnerName.LexicalLess(B->OwnerName));
				return true;
			});
		}
	}

	ResourceListView->RebuildList();

	RefreshSizeMap();
}

TSharedRef<ITableRow> SRenderResourceViewerWidget::HandleResourceGenerateRow(TSharedPtr<FRHIResourceStats> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(RenderResourceViewerInternal::FResourceInfoRow, OwnerTable, Item);
}

void SRenderResourceViewerWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;
	RefreshNodes();
}

FText SRenderResourceViewerWidget::GetResourceSizeText() const
{
	return FText::FromString(RenderResourceViewerInternal::GetFormatedSize(TotalResourceSize));
}

FReply SRenderResourceViewerWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) == true ? FReply::Handled() : FReply::Unhandled();
}

void SRenderResourceViewerWidget::InitCommandList()
{
	CommandList = MakeShared<FUICommandList>();

	const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
	checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));

	CommandList->MapAction(FindInContentBrowserCommand,
		FExecuteAction::CreateSP(this, &SRenderResourceViewerWidget::ContextMenu_FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SRenderResourceViewerWidget::ContextMenu_FindInContentBrowser_CanExecute));
}

TSharedPtr<SWidget> SRenderResourceViewerWidget::OpenContextMenu()
{
	FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

	const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
	checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));

	MenuBuilder.AddMenuEntry
	(
		FindInContentBrowserCommand,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small")
	);

	return MenuBuilder.MakeWidget();
}

void SRenderResourceViewerWidget::ContextMenu_FindInContentBrowser()
{
	UObject* SelectedAsset = nullptr;
	const TArray<TSharedPtr<FRHIResourceStats>>& SelectedNodes = ResourceListView->GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		// Find the UObject asset from the owner name path
		FString ObjectPathString = SelectedNodes[0]->OwnerName.ToString();
		int32 LODIdx = ObjectPathString.Find(TEXT(" [LOD"));
		if (LODIdx > -1)
		{
			ObjectPathString = ObjectPathString.Left(LODIdx);
		}

		SelectedAsset = FSoftObjectPath(ObjectPathString).ResolveObject();
	}

	if (SelectedAsset)
	{
		// Highlight the asset in content browser
		const TArray<UObject*>& Assets = { SelectedAsset };
		const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
	}
}

bool SRenderResourceViewerWidget::ContextMenu_FindInContentBrowser_CanExecute() const
{
	const TArray<TSharedPtr<FRHIResourceStats>>& SelectedNodes = ResourceListView->GetSelectedItems();
	if (!SelectedNodes.IsEmpty())
	{
		for (const TSharedPtr<FRHIResourceStats>& SelectedNode : SelectedNodes)
		{
			if (SelectedNode && !SelectedNode->OwnerName.IsNone())
			{
				return true;
			}
		}
	}
	return false;
}

void SRenderResourceViewerWidget::RefreshSizeMap()
{
	RootTreeMapNode->Children.Empty();

	const URenderResourceViewerSettings* Settings = GetDefault<URenderResourceViewerSettings>();

	RootTreeMapNode->LogicalName = TEXT("Total");
	RootTreeMapNode->Name = TEXT("GPU Memory");
	RootTreeMapNode->Name2 = FString::Printf(TEXT("%s"), *FText::AsMemory(TotalResourceSize).ToString());

	// Build parent nodes in the treemap for all our filters
	TMap<FString, FTreeMapNodeDataPtr> NodeLookup;
	for (const FRenderResourceViewerTreemapFilter& Filter : Settings->Filters)
	{
		FTreeMapNodeDataPtr TreeMapNode = MakeShareable(new FTreeMapNodeData());
		NodeLookup.Add(Filter.FilterString, TreeMapNode);

		TreeMapNode->LogicalName = Filter.FilterString;
		TreeMapNode->Name = Filter.DisplayName;
		TreeMapNode->Size = 0.0f;
		RootTreeMapNode->Children.Add(TreeMapNode);
		TreeMapNode->Parent = RootTreeMapNode.Get();
	}

	// Catch all filter for stuff that doesn't go into our filters.
	FTreeMapNodeDataPtr OtherNode = MakeShareable(new FTreeMapNodeData());
	OtherNode->LogicalName = TEXT("Other");
	OtherNode->Name = TEXT("Other");
	OtherNode->Size = 0.0f;
	RootTreeMapNode->Children.Add(OtherNode);
	OtherNode->Parent = RootTreeMapNode.Get();

	for (TSharedPtr<FRHIResourceStats> Stat : ResourceInfos)
	{
		FTreeMapNodeDataPtr ChildNode = MakeShareable(new FTreeMapNodeData());
		ChildNode->LogicalName = Stat->Name.ToString();
		ChildNode->Name = Stat->Name.ToString();
		ChildNode->Size = Stat->SizeInBytes;
		ChildNode->Name2 = FString::Printf(TEXT("%s"), *FText::AsMemory(Stat->SizeInBytes).ToString());

		bool bAddedToFilter = false;
		for (const FRenderResourceViewerTreemapFilter& Filter : Settings->Filters)
		{
			if (Stat->Name.ToString().StartsWith(Filter.FilterString))
			{
				if (FTreeMapNodeDataPtr* ParentNode = NodeLookup.Find(Filter.FilterString))
				{
					ChildNode->Parent = (*ParentNode).Get();
					(*ParentNode)->Children.Add(ChildNode);
					bAddedToFilter = true;
					break;
				}
			}
		}

		if (!bAddedToFilter)
		{
			// Try to add by owner name.
			if (!Stat->OwnerName.IsNone())
			{
				FString OwnerName = Stat->OwnerName.ToString();
				if (FTreeMapNodeDataPtr* ParentNode = NodeLookup.Find(OwnerName))
				{
					ChildNode->Parent = (*ParentNode).Get();
					(*ParentNode)->Children.Add(ChildNode);
				}
				else
				{
					FTreeMapNodeDataPtr OwnerNode = MakeShareable(new FTreeMapNodeData());
					OwnerNode->LogicalName = OwnerName;
					OwnerNode->Name = OwnerName;
					OwnerNode->Size = Stat->SizeInBytes;

					RootTreeMapNode->Children.Add(OwnerNode);
					OwnerNode->Parent = RootTreeMapNode.Get();

					NodeLookup.Add(OwnerName, OwnerNode);
				}
			}
			else
			{
				// Add it to the other bucket
				ChildNode->Parent = OtherNode.Get();
				OtherNode->Children.Add(ChildNode);
			}
		}
	}

	TreeMapWidget->RebuildTreeMap(false);
	TreeMapWidget->ZoomOut();
}

#undef LOCTEXT_NAMESPACE
