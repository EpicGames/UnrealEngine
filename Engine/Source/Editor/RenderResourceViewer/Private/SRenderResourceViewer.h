// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

struct FRHIResourceStats;
class ITableRow;
class STableViewBase;
class SDockTab;
class FUICommandList;

class SRenderResourceViewerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRenderResourceViewerWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

	SRenderResourceViewerWidget();
	~SRenderResourceViewerWidget() = default;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	void RefreshNodes(bool bUpdateRHIResources = false);
	void RefreshSizeMap();
	TSharedRef<ITableRow> HandleResourceGenerateRow(TSharedPtr<FRHIResourceStats> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const { return SortByColumn == ColumnId ? SortMode : EColumnSortMode::None; }
	FText GetResourceCountText() const									{ return FText::AsNumber(TotalResourceCount); }
	FText GetResourceSizeText() const;
	void FilterTextBox_OnTextChanged(const FText& InFilterText)			{ FilterText = InFilterText; RefreshNodes(); }
	void OnResidentComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)		{ ShowResident = ComboBoxNameToType(NewValue); RefreshNodes(); }	
	void OnTransientComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)	{ ShowTransient = ComboBoxNameToType(NewValue); RefreshNodes(); }
	void OnStreamingComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)	{ ShowStreaming = ComboBoxNameToType(NewValue); RefreshNodes(); }
	void OnRTComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)			{ ShowRT = ComboBoxNameToType(NewValue); RefreshNodes(); }
	void OnDSComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)			{ ShowDS = ComboBoxNameToType(NewValue); RefreshNodes(); }
	void OnUAVComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)			{ ShowUAV = ComboBoxNameToType(NewValue); RefreshNodes(); }
	void OnRTASComboboxChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type)			{ ShowRTAS = ComboBoxNameToType(NewValue); RefreshNodes(); }
	FReply OnRefreshButtonClicked()										{ RefreshNodes(true); return FReply::Handled(); }
	void InitCommandList();
	TSharedPtr<SWidget> OpenContextMenu();
	void ContextMenu_FindInContentBrowser();
	bool ContextMenu_FindInContentBrowser_CanExecute() const;

	enum class EComboBoxType : uint8
	{
		Any,
		Yes,
		No,
	};
	EComboBoxType ComboBoxNameToType(TSharedPtr<FString> Value) const;
	TSharedPtr<FString> ComboBoxTypeToName(EComboBoxType Type) const	{ return ComboBoxNames[(int32)Type]; }
	bool ShouldShow(EComboBoxType FilterType, bool bValue) const;

	TArray<TSharedPtr<FRHIResourceStats>> RHIResources;
	TSharedPtr<SListView<TSharedPtr<FRHIResourceStats>>> ResourceListView;
	TArray<TSharedPtr<FRHIResourceStats>> ResourceInfos;
	FName SortByColumn;
	EColumnSortMode::Type SortMode;
	FText FilterText;
	TSharedPtr< class SEditableTextBox > FilterTextBox;
	uint64 TotalResourceCount = 0;
	uint64 TotalResourceSize = 0;
	TSharedPtr<FUICommandList> CommandList;
	TArray<TSharedPtr<FString>> ComboBoxNames;

	/** Our tree map widget */
	TSharedPtr<class STreeMap> TreeMapWidget;
	/** Our tree map source data */
	TSharedPtr<class FTreeMapNodeData> RootTreeMapNode;

	EComboBoxType ShowResident = EComboBoxType::Any;					// Show resource with Resident flag set
	EComboBoxType ShowTransient = EComboBoxType::No;					// Show resource with Transient flag set
	EComboBoxType ShowStreaming = EComboBoxType::Any;					// Show resource with Streaming flag set
	EComboBoxType ShowRT = EComboBoxType::Any;							// Show resource with RenderTarget flag set
	EComboBoxType ShowDS = EComboBoxType::Any;							// Show resource with DepthStencil flag set
	EComboBoxType ShowUAV = EComboBoxType::Any;							// Show resource with UAV flag set
	EComboBoxType ShowRTAS = EComboBoxType::Any;						// Show resource with RayTracingAccelationStructure flag set
};
