// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class SRenderResourceViewerWidget;
class SDockTab;
class FSpawnTabArgs;

/**
 * Struct Viewer module
 */
class FRenderResourceViewerModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

private:
	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& Args);

	void AssignWindow(const TSharedRef<SRenderResourceViewerWidget>& InWindow)
	{
		RenderResourceViewerWindow = InWindow;
	}

	TWeakPtr<class SRenderResourceViewerWidget> RenderResourceViewerWindow;
};
