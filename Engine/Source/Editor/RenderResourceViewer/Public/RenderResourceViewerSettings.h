// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "RenderResourceViewerSettings.generated.h"

USTRUCT()
struct FRenderResourceViewerTreemapFilter
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, Category = "Section")
	FString FilterString;

	UPROPERTY(config, EditAnywhere, Category = "Section")
	FString DisplayName;
};

UCLASS(MinimalAPI, config = Editor, DefaultConfig, meta = (DisplayName = "Render Resource Viewer"))
class URenderResourceViewerSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Config, EditAnywhere, Category = "Treemap")
	TArray<FRenderResourceViewerTreemapFilter> Filters;
};
