// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class RenderResourceViewer : ModuleRules
{
	public RenderResourceViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"WorkspaceMenuStructure",
				"ContentBrowser",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"SlateCore",
				"Slate",
				"RHI",
				"DeveloperSettings",
				"TreeMap",
				"UnrealEd"
			}
		);
	}
}
